{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "NVIDIA Rivermax Media synchronization for nDisplay",
	"Description": "Adding NVIDIA Rivermax synchronization capabilities for nDisplay",
	"Category": "Virtual Production",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"Hidden": "false",
	"CanContainContent": false,
	"IsBetaVersion": true,
	"Installed": false,
	"SupportedTargetPlatforms": [ "Win64", "Linux" ],
	"Plugins": [
		{
			"Name": "nDisplay",
			"Enabled": true
		},
		{
			"Name": "RivermaxCore",
			"Enabled": true
		},
		{
			"Name": "RivermaxMedia",
			"Enabled": true
		}
	],
	"Modules": [
		{
			"Name": "RivermaxSync",
			"Type": "Runtime",
			"LoadingPhase": "Default",
			"PlatformAllowList": [ "Win64", "Linux" ]
		},
		{
			"Name": "RivermaxSyncEditor",
			"Type": "Editor",
			"LoadingPhase": "Default",
			"PlatformAllowList": [ "Win64", "Linux" ]
		}
	]
}