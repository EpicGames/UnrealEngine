// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaOutputSynchronizationPolicyRivermax.h"

#include "Async/Async.h"
#include "Cluster/IDisplayClusterClusterManager.h"
#include "Config/IDisplayClusterConfigManager.h"
#include "HAL/IConsoleManager.h"
#include "IDisplayCluster.h"
#include "IRivermaxCoreModule.h"
#include "IRivermaxManager.h"
#include "RivermaxMediaCapture.h"
#include "RivermaxPTPUtils.h"
#include "RivermaxSyncLog.h"


namespace UE::RivermaxSync
{
	static TAutoConsoleVariable<float> CVarRivermaxSyncWakeupOffset(
		TEXT("Rivermax.Sync.WakeUpOffset"), 0.5f,
		TEXT("Offset from alignment point to wake up at when barrier stalls the cluster. Units: milliseconds"),
		ECVF_Default);

	static TAutoConsoleVariable<bool> CVarRivermaxSyncEnableSelfRepair(
		TEXT("Rivermax.Sync.EnableSelfRepair"), true,
		TEXT("Whether to use exchanged data in the synchronization barrier to detect desynchronized state and act on it to self repair"),
		ECVF_Default);

	static TAutoConsoleVariable<int32> CVarRivermaxPtpUnsyncFramesPerReport(
		TEXT("Rivermax.Sync.Ptp.UnsyncFramesPerReport"), 120,
		TEXT("When there are PTP mismatches in the cluster, Stage Monitor events are issued.\n"
			 "PTP mismatches are a stable condition and this cvar controls how often to send the events.\n"
			 "Use -1 (or any negative number) to disable these reports."),
		ECVF_Default);

	static bool GbTriggerRandomDesync = false;
	FAutoConsoleVariableRef CVarTriggerRandomDesync(
		TEXT("Rivermax.Sync.ForceDesync")
		, UE::RivermaxSync::GbTriggerRandomDesync
		, TEXT("After barrier synchronization, trigger random stall."), ECVF_Cheat);

}


FMediaOutputSynchronizationPolicyRivermaxHandler::FMediaOutputSynchronizationPolicyRivermaxHandler(UMediaOutputSynchronizationPolicyRivermax* InPolicyObject)
	: Super(InPolicyObject)
	, MarginMs(InPolicyObject->MarginMs)
{
	// Allocate memory to store the data exchanged in the barrier
	BarrierData.SetNumZeroed(sizeof(FMediaSyncBarrierData));
}

TSubclassOf<UDisplayClusterMediaOutputSynchronizationPolicy> FMediaOutputSynchronizationPolicyRivermaxHandler::GetPolicyClass() const
{
	return UMediaOutputSynchronizationPolicyRivermax::StaticClass();
}

bool FMediaOutputSynchronizationPolicyRivermaxHandler::IsCaptureTypeSupported(UMediaCapture* MediaCapture) const
{
	// We need to make sure:
	// - it's RivermaxCapture
	// - it uses PTP or System time source
	// - it uses AlignmentPoint alignment mode
	if (URivermaxMediaCapture* RmaxCapture = Cast<URivermaxMediaCapture>(MediaCapture))
	{
		using namespace UE::RivermaxCore;

		if (TSharedPtr<IRivermaxManager> RivermaxMgr = IRivermaxCoreModule::Get().GetRivermaxManager())
		{
			const ERivermaxTimeSource TimeSource = RivermaxMgr->GetTimeSource();

			if (TimeSource == ERivermaxTimeSource::PTP || TimeSource == ERivermaxTimeSource::System)
			{
				const FRivermaxOutputOptions Options = RmaxCapture->GetOutputOptions();

				if (Options.AlignmentMode == ERivermaxAlignmentMode::AlignmentPoint)
				{
					return true;
				}
			}
		}
	}

	return false;
}

void FMediaOutputSynchronizationPolicyRivermaxHandler::Synchronize()
{
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(RmaxSync::Synchronize);

		// Sync on the barrier if everything is good
		if (!IsRunning())
		{
			UE_LOGF(LogRivermaxSync, Warning, "'%ls': Synchronization is off", *GetMediaDeviceId());
			return;
		}

		TSharedPtr<IDisplayClusterGenericBarriersClient> BarrierClient = GetBarrierClient();
		if (!BarrierClient)
		{
			UE_LOGF(LogRivermaxSync, Warning, "'%ls': Barrier client is nullptr", *GetMediaDeviceId());
			return;
		}

		UE_LOGF(LogRivermaxSync, Verbose, "'%ls': Synchronizing caller '%ls' at the barrier '%ls'", *GetMediaDeviceId(), *GetThreadMarker(), *GetBarrierId());

		URivermaxMediaCapture* RmaxCapture = Cast<URivermaxMediaCapture>(CapturingDevice);
		if (RmaxCapture == nullptr || RmaxCapture->GetState() != EMediaCaptureState::Capturing)
		{
			UE_LOGF(LogRivermaxSync, Warning, "'%ls': Rivermax Capture isn't valid or not capturing", *GetMediaDeviceId());
			return;
		}

		// Verify if we are safe to go inside the barrier.
		{
			// Ask the sync implementation about how much time we have before next synchronization timepoint
			const double TimeLeftSeconds = GetTimeBeforeNextSyncPoint();
			// Convert to seconds
			const double MarginSeconds = double(MarginMs) / 1000;

			// In case we're unsafe, skip the upcoming sync timepoint
			if (TimeLeftSeconds < MarginSeconds)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(RmaxSync::MarginProtection);

				const float OffsetTimeSeconds = UE::RivermaxSync::CVarRivermaxSyncWakeupOffset.GetValueOnAnyThread() * 1E-3;
				// Sleep for a bit longer to skip the alignment timepoint
				const float SleepTime = TimeLeftSeconds + OffsetTimeSeconds;

				UE_LOGF(LogRivermaxSync, VeryVerbose, "'%ls': TimeLeft(%lf) < Margin(%lf) --> Sleeping for %lf...",
					*GetMediaDeviceId(), TimeLeftSeconds, MarginSeconds, SleepTime);

				FPlatformProcess::SleepNoStats(SleepTime);
			}
		}


		// We are good to go in the barrier, prepare payload about presented frame
		UE::RivermaxCore::FPresentedFrameInfo FrameInfo;
		RmaxCapture->GetLastPresentedFrameInformation(FrameInfo);

		// Fill the memory to be exchanged by nodes in the barrier.
		BarrierDataStruct.InsertFrameInfo(FrameInfo);
		UE_LOGF(LogRivermaxSync, VeryVerbose, "'%ls' Entering with %u", *GetMediaDeviceId(), BarrierDataStruct.LastRenderedFrameNumber[0]);
		FMemory::Memcpy(BarrierData.GetData(), &BarrierDataStruct, sizeof(FMediaSyncBarrierData));

		// We don't use response data for now
		TArray<uint8> ResponseData;

		// Synchronize on a barrier
		BarrierClient->Synchronize(GetBarrierId(), GetThreadMarker(), BarrierData, ResponseData);
	}

	// Debug cvar to potentially stall a node after exiting the barrier and missing alignment points
	if (UE::RivermaxSync::GbTriggerRandomDesync)
	{
		using namespace UE::RivermaxCore;

		FRandomStream RandomStream(FPlatformTime::Cycles64());
		const bool bTriggerDesync = (RandomStream.FRandRange(0.0, 1.0) > 0.7) ? true : false;

		URivermaxMediaCapture* RmaxCapture = Cast<URivermaxMediaCapture>(CapturingDevice);
		FRivermaxOutputOptions OutputOptions = RmaxCapture->GetOutputOptions();

		// Currently we only support synchronization with Video streams.
		TSharedPtr<FRivermaxVideoOutputOptions> VideoOptions = OutputOptions.GetStreamOptions<FRivermaxVideoOutputOptions>(ERivermaxStreamType::ST2110_20);

		if (!ensure(VideoOptions.IsValid()))
		{
			return;
		}

		if (bTriggerDesync)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(RmaxSync::ForceBadSync);
			double TimeLeftSeconds = GetTimeBeforeNextSyncPoint();
			TimeLeftSeconds += VideoOptions->FrameRate.AsInterval();
			const float OffsetTimeSeconds = UE::RivermaxSync::CVarRivermaxSyncWakeupOffset.GetValueOnAnyThread() * 1E-3;
			const float SleepTime = TimeLeftSeconds + OffsetTimeSeconds;
			FPlatformProcess::SleepNoStats(SleepTime);
		}

		UE::RivermaxSync::GbTriggerRandomDesync = false;
	}
}

double FMediaOutputSynchronizationPolicyRivermaxHandler::GetTimeBeforeNextSyncPoint() const
{
	if (URivermaxMediaCapture* RmaxCapture = Cast<URivermaxMediaCapture>(CapturingDevice))
	{
		if (RmaxCapture->GetState() == EMediaCaptureState::Capturing)
		{
			using namespace UE::RivermaxCore;

			if (TSharedPtr<IRivermaxManager> RivermaxMgr = IRivermaxCoreModule::Get().GetRivermaxManager())
			{
				// Get current time
				const uint64 CurrentTimeNanosec = RivermaxMgr->GetTime();

				// Get next alignment timepoint
				FRivermaxOutputOptions Options = RmaxCapture->GetOutputOptions();

				TSharedPtr<FRivermaxVideoOutputOptions> VideoOptions = Options.GetStreamOptions<FRivermaxVideoOutputOptions>(ERivermaxStreamType::ST2110_20);

				if (!ensure(VideoOptions.IsValid()))
				{
					return 1.f;
				}

				const uint64 NextAlignmentTimeNanosec = GetNextAlignmentPoint(CurrentTimeNanosec, VideoOptions->FrameRate);

				// Time left
				checkSlow(NextAlignmentTimeNanosec > CurrentTimeNanosec);
				const uint64 TimeLeftNanosec = NextAlignmentTimeNanosec - CurrentTimeNanosec;

				// Return remaining time in seconds
				return double(TimeLeftNanosec * 1E-9);
			}
		}
	}

	// Normally we should never get here. As a fallback approach, return some big time interval
	// to prevent calling thread blocking. 1 second is more than any possible threshold.
	return 1.f;
}

bool FMediaOutputSynchronizationPolicyRivermaxHandler::InitializeBarrier(const FString& SyncInstanceId)
{
	// Base initialization first
	if (!Super::InitializeBarrier(SyncInstanceId))
	{
		UE_LOGF(LogRivermaxSync, Warning, "Couldn't initialize barrier for '%ls'", *GetMediaDeviceId());
		return false;
	}

	// Get barrier client
	TSharedPtr<IDisplayClusterGenericBarriersClient> BarrierClient = GetBarrierClient();
	if (!BarrierClient)
	{
		UE_LOGF(LogRivermaxSync, Warning, "Couldn't access a barrier client for '%ls'", *GetMediaDeviceId());
		return false;
	}

	// Get delegate bound to the specific barrier
	IDisplayClusterGenericBarriersClient::FOnGenericBarrierSynchronizationDelegate* Delegate = BarrierClient->GetBarrierSyncDelegate(GetBarrierId());
	if (!Delegate)
	{
		UE_LOGF(LogRivermaxSync, Warning, "'%ls': Couldn't access a barrier delegate for barrier '%ls'", *GetMediaDeviceId(), *GetBarrierId());
		return false;
	}

	// Setup synchronization delegate that will be called on the p-node
	Delegate->BindRaw(this, &FMediaOutputSynchronizationPolicyRivermaxHandler::HandleBarrierSync);

	return true;
}


bool FMediaOutputSynchronizationPolicyRivermaxHandler::FMediaSyncBarrierData::HasConfirmedDesync(const FMediaSyncBarrierData& OtherBarrierData, int64& OutVsyncDelta) const
{
	OutVsyncDelta = 0;

	for (int32 FirstNodeIdx = 0; FirstNodeIdx < FMediaSyncBarrierData::FRAMEHISTORYLEN; ++FirstNodeIdx)
	{
		const uint32 FirstNodeFrameNumber = LastRenderedFrameNumber[FirstNodeIdx];
		const uint64 FirstNodeVsyncBoundary = PresentedFrameBoundaryNumber[FirstNodeIdx];

		for (int32 OtherNodeIdx = 0; OtherNodeIdx < FMediaSyncBarrierData::FRAMEHISTORYLEN; ++OtherNodeIdx)
		{
			const uint32 OtherNodeFrameNumber = OtherBarrierData.LastRenderedFrameNumber[OtherNodeIdx];
			const uint64 OtherNodeVsyncBoundary = OtherBarrierData.PresentedFrameBoundaryNumber[OtherNodeIdx];

			const bool bSameFrame = (FirstNodeFrameNumber == OtherNodeFrameNumber);
			const bool bSameVsync = (FirstNodeVsyncBoundary == OtherNodeVsyncBoundary);

			// Keep track of the maximum Vsync delta of equal frames.
			if (bSameFrame && !bSameVsync)
			{
				// Positive means that the node has a PTP frame number larger than the base we're comparing with.
				const int64 VsyncDelta = static_cast<int64>(OtherNodeVsyncBoundary) - static_cast<int64>(FirstNodeVsyncBoundary);

				if (FMath::Abs(VsyncDelta) > FMath::Abs(OutVsyncDelta))
				{
					OutVsyncDelta = VsyncDelta;
				}
			}

			// If they agree on a recent frame, consider them in sync
			if (bSameFrame && bSameVsync)
			{
				return false;
			}

			// If they presented a different frame on the same vsync, they are out of sync
			if (bSameVsync && !bSameFrame)
			{
				return true;
			}
		}
	}

	// If we could not confirm a desync, we conservatively not consider a desync to have been detected.
	return false;
}


FString FMediaOutputSynchronizationPolicyRivermaxHandler::FMediaSyncBarrierData::LastRenderedFrameNumbersAsString() const
{
	FString FrameNumbers;

	for (int32 Idx = 0; Idx < FRAMEHISTORYLEN; ++Idx)
	{
		FrameNumbers += FString::Printf(TEXT("%u"), LastRenderedFrameNumber[Idx]);

		if (Idx < FRAMEHISTORYLEN - 1)
		{
			FrameNumbers += TEXT(", ");
		}
	}

	return FrameNumbers;
}

FString FMediaOutputSynchronizationPolicyRivermaxHandler::FMediaSyncBarrierData::PresentedFrameBoundaryNumbersAsString() const
{
	FString BoundaryNumbers;

	for (int32 Idx = 0; Idx < FRAMEHISTORYLEN; ++Idx)
	{
		BoundaryNumbers += FString::Printf(TEXT("%llu"), PresentedFrameBoundaryNumber[Idx]);

		if (Idx < FRAMEHISTORYLEN - 1)
		{
			BoundaryNumbers += TEXT(", ");
		}
	}

	return BoundaryNumbers;
}

bool FMediaOutputSynchronizationPolicyRivermaxHandler::PickPtpBaseNodeAndData(
	const FGenericBarrierSynchronizationDelegateData& BarrierSyncData, 
	const FMediaSyncBarrierData*& OutPtpBaseNodeData,
	FString& OutBaseNodeId
) const
{
	// Identify the name of the primary node, which will be the default for PTP mismatch measurements.

	OutPtpBaseNodeData = nullptr;

	// We need at least one node.
	if (BarrierSyncData.RequestData.Num() <= 0)
	{
		return false;
	}

	IDisplayClusterConfigManager* ConfigMgr = IDisplayCluster::Get().GetConfigMgr();

	if (!ConfigMgr)
	{
		return false;
	}

	OutBaseNodeId = ConfigMgr->GetPrimaryNodeId();
	const TArray<uint8>* PrimaryDataRaw = BarrierSyncData.RequestData.Find(OutBaseNodeId);

	if (PrimaryDataRaw)
	{
		check(PrimaryDataRaw->Num() == sizeof(FMediaSyncBarrierData));
		OutPtpBaseNodeData = reinterpret_cast<const FMediaSyncBarrierData*>(PrimaryDataRaw->GetData());

		return true;
	}

	// If the primary node isn't in the barrier, then we pick the first one in a sorted list of node ids.
	// We don't need to actually sort the list, just find the first one, which is O(n) instead of O(n log n).

	const FString* FirstKeyIfSorted = nullptr;
	const TArray<uint8>* FirstValueIfSorted = nullptr;

	for (const auto& Elem : BarrierSyncData.RequestData)
	{
		const FString& Key = Elem.Key;
		const TArray<uint8>& Value = Elem.Value;

		if (!FirstKeyIfSorted || Key < *FirstKeyIfSorted)
		{
			FirstKeyIfSorted = &Key;
			FirstValueIfSorted = &Value;
		}
	}

	check(FirstKeyIfSorted && FirstValueIfSorted);

	OutBaseNodeId = *FirstKeyIfSorted;

	check(FirstValueIfSorted->Num() == sizeof(FMediaSyncBarrierData));
	OutPtpBaseNodeData = reinterpret_cast<const FMediaSyncBarrierData*>(FirstValueIfSorted->GetData());

	return true;
}

void FMediaOutputSynchronizationPolicyRivermaxHandler::HandleBarrierSync(FGenericBarrierSynchronizationDelegateData& BarrierSyncData)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(RmaxSync::BarrierSync);

	// Nothing to do if there is no barrier data.
	if (BarrierSyncData.RequestData.Num() <= 0)
	{
		UE_LOGF(LogRivermaxSync, Verbose, "'%ls': No data was provided by nodes for sync barrier.", *GetMediaDeviceId())
		return;
	}

	// Deterministically pick a node in the cluster to use as the PTP base for mismatch detections.

	const FMediaSyncBarrierData* PtpBaseNodeData = nullptr;
	FString PtpBaseNodeId;

	if (!PickPtpBaseNodeAndData(BarrierSyncData, PtpBaseNodeData, PtpBaseNodeId))
	{
		UE_LOGF(LogRivermaxSync, Warning, "Could not find a base node for ptp mismatch comparisons");
		return;
	}

	check(PtpBaseNodeData);

	bool bSelfRepairRequired = false; // True if there is any need (and benefit) in initiating a sync repair action.

	TMap<FString, int64> PtpMismatchedNodes; // Collects node ids with mismatched PTP frames.

	// To avoid flooding the network and the receiver with these events, we only report ptp mismatches every few frames.
	const int32 PtpUnsyncFramesPerReport = UE::RivermaxSync::CVarRivermaxPtpUnsyncFramesPerReport.GetValueOnAnyThread();
	const bool bShouldReportPtpMismatches = (PtpUnsyncFramesPerReport > 0) && !(PtpBaseNodeData->LastRenderedFrameNumber[0] % PtpUnsyncFramesPerReport);

	// Iterate over the presentation requests and detect PTP de-syncs.
	for (const TPair<FString, TArray<uint8>>& FramePresentedInfo : BarrierSyncData.RequestData)
	{
		const FString& ThreadPresentedFrame = FramePresentedInfo.Key;

		// Find the corresponding cluster node based on the synchronization thread name from request
		const FString* NodePresentedFrame = BarrierSyncData.ThreadToNodeMap.Find(ThreadPresentedFrame);
		check(NodePresentedFrame);

		// Don't leave nullptr just in case (this is not expected)
		if (!NodePresentedFrame)
		{
			NodePresentedFrame = &ThreadPresentedFrame;
		}

		// Barrier data with unexpected sizes are a logical error.
		check(FramePresentedInfo.Value.Num() == sizeof(FMediaSyncBarrierData));

		// Get the node data in struct format.
		const FMediaSyncBarrierData* const NodeData = reinterpret_cast<const FMediaSyncBarrierData* const>(FramePresentedInfo.Value.GetData());

		// Skip base node comparing with itself
		if (NodeData == PtpBaseNodeData)
		{
			continue;
		}

		// We expect all nodes to enter the barrier after presenting the SAME frame at the SAME frame number.
		// If a node enters the barrier a frame late on the other, self repair will be triggered.
		// Same thing goes if all nodes didn't present the same frame

		int64 VsyncDelta = 0;

		if (PtpBaseNodeData->HasConfirmedDesync(*NodeData, VsyncDelta))
		{
			bSelfRepairRequired = true;

			UE_LOGF(LogRivermaxSync, Warning,
				"Desync detected: Node '%ls' presented frames (%ls) at boundaries (%ls), but node '%ls' presented frames (%ls) at boundaries (%ls)",
				*PtpBaseNodeId,
				*PtpBaseNodeData->LastRenderedFrameNumbersAsString(),
				*PtpBaseNodeData->PresentedFrameBoundaryNumbersAsString(),
				**NodePresentedFrame,
				*NodeData->LastRenderedFrameNumbersAsString(),
				*NodeData->PresentedFrameBoundaryNumbersAsString());

			// We do not break the loop, in order to log all timing issues detected in the current frame.
		}
		else if (VsyncDelta != 0)
		{
			UE_LOGF(LogRivermaxSync, Warning,
				"Frames not presented at the same PTP frame boundary: Node '%ls' presented frames (%ls) at boundaries (%ls), but node '%ls' presented frames (%ls) at boundaries (%ls)",
				*PtpBaseNodeId,
				*PtpBaseNodeData->LastRenderedFrameNumbersAsString(),
				*PtpBaseNodeData->PresentedFrameBoundaryNumbersAsString(),
				**NodePresentedFrame,
				*NodeData->LastRenderedFrameNumbersAsString(),
				*NodeData->PresentedFrameBoundaryNumbersAsString());
		}

		// Collect Vsync deltas for reporting purposes.
		if (bShouldReportPtpMismatches && VsyncDelta)
		{
			PtpMismatchedNodes.Add(*NodePresentedFrame, VsyncDelta);
		}
	}

	// Report to Stage Monitor the PTP mismatches.
	if (bShouldReportPtpMismatches)
	{
		AsyncTask(ENamedThreads::GameThread, [PtpMismatchedNodes, PtpBaseNodeId]()
			{
				IStageDataProvider::SendMessage<FRivermaxClusterPtpUnsyncEvent>(
					EStageMessageFlags::None, // Doesn't need to be reliable.
					PtpMismatchedNodes,
					PtpBaseNodeId
				);
			});
	}

	// This cvar can disable the self repair.
	const bool bCanUseSelfRepair = UE::RivermaxSync::CVarRivermaxSyncEnableSelfRepair.GetValueOnAnyThread();

	// If repair is required, we stall until we are past the next alignment point to have all scheduler present something and get closer to a synchronized state.
	if (bSelfRepairRequired)
	{
		// Only trigger the self repair when allowed. It consists on sleeping until the next ptp vsync passes.
		if (bCanUseSelfRepair)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(RmaxSync::SelfRepair);

			const double TimeLeftSeconds = GetTimeBeforeNextSyncPoint();
			const float OffsetTimeSeconds = UE::RivermaxSync::CVarRivermaxSyncWakeupOffset.GetValueOnAnyThread() * 1E-3;
			const float SleepTime = TimeLeftSeconds + OffsetTimeSeconds;
			FPlatformProcess::SleepNoStats(SleepTime);
		}
	}
	else
	{
		// VeryVerbose log of the ptp frame presentation values when everything is in ptp sync.

		UE_LOGF(LogRivermaxSync, VeryVerbose, "'%ls': Cluster likely synchronized (no confirmed desync). ptp base node '%ls' presented frame %u at frame boundary %llu", 
			*GetMediaDeviceId(),
			*PtpBaseNodeId,
			PtpBaseNodeData->LastRenderedFrameNumber[0],
			PtpBaseNodeData->PresentedFrameBoundaryNumber[0]
		);
	}
}

TSharedPtr<IDisplayClusterMediaOutputSynchronizationPolicyHandler> UMediaOutputSynchronizationPolicyRivermax::GetHandler()
{
	if (!Handler)
	{
		Handler = MakeShared<FMediaOutputSynchronizationPolicyRivermaxHandler>(this);
	}

	return Handler;
}


FString FRivermaxClusterPtpUnsyncEvent::ToString() const
{
	if (NodePtpFrameDeltas.Num() > 0)
	{
		FString Result = FString::Printf(TEXT("PTP video frame mismatches compared to PTP base node '%s' on nodes: "), *PtpBaseNodeId);

		// Sort the keys to produce a deterministic string output, which will help readability of repeated events.

		TArray<FString> SortedKeys;
		NodePtpFrameDeltas.GetKeys(SortedKeys);
		SortedKeys.Sort();

		// Iterate over sorted keys for deterministic output
		for (int32 NodeIdx = 0; NodeIdx < SortedKeys.Num(); ++NodeIdx)
		{
			const FString& Key = SortedKeys[NodeIdx];
			int64 PtpFrameDelta = NodePtpFrameDeltas[Key];

			Result += FString::Printf(TEXT("%s(%lld)"), *Key, PtpFrameDelta);

			if (NodeIdx < SortedKeys.Num() - 1)
			{
				Result += TEXT(", ");
			}
		}

		return Result;
	}

	return FString(TEXT("All nodes are in PTP sync."));
}
