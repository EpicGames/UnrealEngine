// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Synchronization/DisplayClusterMediaOutputSynchronizationPolicyEthernetBarrierBase.h"

#include "IRivermaxOutputStream.h"
#include "IStageDataProvider.h"

#include "MediaOutputSynchronizationPolicyRivermax.generated.h"

struct FGenericBarrierSynchronizationDelegateData;


/*
 * Synchronization logic handler class for UMediaOutputSynchronizationPolicyRivermax.
 */
class RIVERMAXSYNC_API FMediaOutputSynchronizationPolicyRivermaxHandler
	: public FDisplayClusterMediaOutputSynchronizationPolicyEthernetBarrierBaseHandler
{
	using Super = FDisplayClusterMediaOutputSynchronizationPolicyEthernetBarrierBaseHandler;
public:

	FMediaOutputSynchronizationPolicyRivermaxHandler(UMediaOutputSynchronizationPolicyRivermax* InPolicyObject);

	//~ Begin IDisplayClusterMediaOutputSynchronizationPolicyHandler interface
	virtual TSubclassOf<UDisplayClusterMediaOutputSynchronizationPolicy> GetPolicyClass() const override;
	//~ End IDisplayClusterMediaOutputSynchronizationPolicyHandler interface

public:
	/** We do our own synchronization by looking at distance to alignment point. */
	virtual void Synchronize() override;

	/** Returns true if specified media capture type can be synchonized by the policy implementation */
	virtual bool IsCaptureTypeSupported(UMediaCapture* MediaCapture) const override;

protected:

	/** Holds data provided to server by each node when joining the barrier */
	struct FMediaSyncBarrierData
	{
		FMediaSyncBarrierData()
		{
			Reset();
		}

		/** Reset the data to default values */
		void Reset()
		{
			for (int32 Idx = 0; Idx < FRAMEHISTORYLEN; ++Idx)
			{
				PresentedFrameBoundaryNumber[Idx] = 0;
				LastRenderedFrameNumber[Idx] = 0;
			}
		}

		/** Insert the given frame information into the recorded presentation history */
		void InsertFrameInfo(const UE::RivermaxCore::FPresentedFrameInfo& FrameInfo)
		{
			// Shift existing history entries
			for (int32 Idx = FRAMEHISTORYLEN - 1; Idx > 0; --Idx)
			{
				PresentedFrameBoundaryNumber[Idx] = PresentedFrameBoundaryNumber[Idx - 1];
				LastRenderedFrameNumber[Idx] = LastRenderedFrameNumber[Idx - 1];
			}

			// Insert new frame info at the beginning
			PresentedFrameBoundaryNumber[0] = FrameInfo.PresentedFrameBoundaryNumber;
			LastRenderedFrameNumber[0] = FrameInfo.RenderedFrameNumber;
		}

		/** Rendered frames as comma separated string */
		FString LastRenderedFrameNumbersAsString() const;

		/** Presented frame boundaries as comma separated string */
		FString PresentedFrameBoundaryNumbersAsString() const;

		/**
		 * Returns true if the frame presentation history indicates a desynced state.
		 *
		 * @param OtherBarrierData
		 *     Barrier data of the node we're comparing with.
		 *
		 * @param OutVsyncDelta
		 *     When the same frame is presented at different Vsync frame boundaries, this parameter
		 *     contains the delta between them. Useful in detecting large PTP differences between nodes.
		 *
		 * @return
		 *     True if different frames were presented at the same Vsync frame boundaries.
		 */
		bool HasConfirmedDesync(const FMediaSyncBarrierData& OtherBarrierData, int64& OutVsyncDelta) const;

		/** How many frames to include in the history */
		static constexpr int32 FRAMEHISTORYLEN = 2;

		/** Frame boundary number at which the last frame was presented.  */
		uint64 PresentedFrameBoundaryNumber[FRAMEHISTORYLEN];

		/** Last engine frame number that was presented */
		uint32 LastRenderedFrameNumber[FRAMEHISTORYLEN];
	};

protected:

	/** Initializes dynamic barrier on the primary node. */
	virtual bool InitializeBarrier(const FString& SyncInstanceId) override;

	/** Barrier callback containing data from each node to detect if cluster is out of sync. */
	void HandleBarrierSync(FGenericBarrierSynchronizationDelegateData& BarrierSyncData);

	/** Returns amount of time before next synchronization point. */
	double GetTimeBeforeNextSyncPoint() const;

	/** 
	 * Deterministically picks a node to base ptp offsets on other nodes from.
	 * 
	 * @param BarrierSyncData the barrier data from all the clients.
	 * @param OutPtpBaseNodeData Will be populated with a pointer to the barrier sync data of the selected ptp base node.
	 * @param OutPtpBaseNodeId For convenience, this returns the name of the ptp base node.
	 */
	bool PickPtpBaseNodeAndData(
		const FGenericBarrierSynchronizationDelegateData& BarrierSyncData,
		const FMediaSyncBarrierData*& OutPtpBaseNodeData,
		FString& OutPtpBaseNodeId
	) const;

protected:

	/** Holds data provided to server by this node when joining the barrier */
	struct FMediaSyncBarrierData BarrierDataStruct;

	/** Synchronization margin (ms) */
	float MarginMs = 5.0f;

	/** Memory buffer used to contain data exchanged in the barrier */
	TArray<uint8> BarrierData;
};


/*
 * Rivermax media synchronization policy implementation
 */
UCLASS(editinlinenew, Blueprintable, meta = (DisplayName = "Rivermax (PTP)"))
class RIVERMAXSYNC_API UMediaOutputSynchronizationPolicyRivermax
	: public UDisplayClusterMediaOutputSynchronizationPolicyEthernetBarrierBase
{
	GENERATED_BODY()


public:
	virtual TSharedPtr<IDisplayClusterMediaOutputSynchronizationPolicyHandler> GetHandler() override;

protected:
	TSharedPtr<IDisplayClusterMediaOutputSynchronizationPolicyHandler> Handler;

public:
	/** Synchronization margin (ms) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings", meta = (DisplayName = "Margin (ms)", ClampMin = "1", ClampMax = "20", UIMin = "1", UIMax = "20"))
	float MarginMs = 5.0f;
};

/**
 * Stage Monitor event to report nodes that are out of PTP sync with respect to a given PTP base node.
 */
USTRUCT()
struct FRivermaxClusterPtpUnsyncEvent : public FStageProviderEventMessage
{
	GENERATED_BODY()

public:
	FRivermaxClusterPtpUnsyncEvent() = default;

	FRivermaxClusterPtpUnsyncEvent(const TMap<FString, int64>& InNodePtpFrameDeltas, const FString& InPtpBaseNodeId)
		: NodePtpFrameDeltas(InNodePtpFrameDeltas)
		, PtpBaseNodeId(InPtpBaseNodeId)
	{}

public:

	/** Nodes with PTP video frame mismatches compared to the ptp base node id. */
	UPROPERTY(VisibleAnywhere, Category = "PtpSync")
	TMap<FString, int64> NodePtpFrameDeltas;

	/** Id of the base node the PTP delta video frames are compared with */
	UPROPERTY(VisibleAnywhere, Category = "PtpSync")
	FString PtpBaseNodeId;

public:

	//~ Begin FStageDataBaseMessage
	virtual FString ToString() const override;
	//~ End FStageDataBaseMessage
};

