// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaOutputSynchronizationPolicyRivermaxFactory.h"

#include "AssetTypeCategories.h"

#include "MediaOutputSynchronizationPolicyRivermax.h"


/**
 * UMediaOutputSynchronizationPolicyRivermax
 */
UMediaOutputSynchronizationPolicyRivermaxFactory::UMediaOutputSynchronizationPolicyRivermaxFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UMediaOutputSynchronizationPolicyRivermax::StaticClass();
	bCreateNew = true;
	bEditAfterNew = true;
}

UObject* UMediaOutputSynchronizationPolicyRivermaxFactory::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	return NewObject<UMediaOutputSynchronizationPolicyRivermax>(InParent, InClass, InName, Flags);
}

uint32 UMediaOutputSynchronizationPolicyRivermaxFactory::GetMenuCategories() const
{
	return EAssetTypeCategories::None;
}

bool UMediaOutputSynchronizationPolicyRivermaxFactory::ShouldShowInNewMenu() const
{
	return true;
}
