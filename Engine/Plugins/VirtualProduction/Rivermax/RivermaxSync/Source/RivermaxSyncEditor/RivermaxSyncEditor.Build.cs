// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class RivermaxSyncEditor : ModuleRules
{
	public RivermaxSyncEditor(ReadOnlyTargetRules ROTargetRules) : base(ROTargetRules)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"Engine",
				"RivermaxSync",
				"UnrealEd"
			});
	}
}
